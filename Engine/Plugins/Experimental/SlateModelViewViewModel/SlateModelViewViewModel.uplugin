{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Slate Model View Viewmodel",
	"Description": "Implementation of the Model View Viewmodel pattern for Slate",
	"Category": "UI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SlateMVVM",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
