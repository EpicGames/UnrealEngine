// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateViewModelBindings.h"

namespace UE::Slate::MVVM
{

namespace Private
{

/**
 *
 */
class FBinding
{
public:
	FBinding(UE::FieldNotification::FFieldId WhenChanged, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate)
		//: DelegateHandle(Delegate.GetHandle()) // because he user won't be able to know the handle
		: CompleteDelegate(MoveTemp(Delegate))
		, FieldId(WhenChanged)
	{
	}
	FBinding(UE::FieldNotification::FFieldId WhenChanged, FSimpleDelegate Delegate)
		//: DelegateHandle(Delegate.GetHandle()) // because he user won't be able to know the handle
		: SimpleDelegate(MoveTemp(Delegate))
		, FieldId(WhenChanged)
	{
	}

	UE::FieldNotification::FFieldId GetFieldId() const
	{
		return FieldId;
	}

	void Execute(UObject* Object)
	{
		SimpleDelegate.ExecuteIfBound();
		CompleteDelegate.ExecuteIfBound(Object, FieldId);
	}

private:
	//FDelegateHandle DelegateHandle;
	INotifyFieldValueChanged::FFieldValueChangedDelegate CompleteDelegate;
	FSimpleDelegate SimpleDelegate;
	UE::FieldNotification::FFieldId FieldId;
};

/**
*
*/
class FSource
{
public:
	FWeakObjectPtr Source; //TScriptInterface<INotifyFieldValueChanged>
	TArray<TUniquePtr<FBinding>> Bindings;
	TMap<UE::FieldNotification::FFieldId, FDelegateHandle> Delegates;

public:
	FSource(TScriptInterface<INotifyFieldValueChanged> InSource)
		: Source(InSource.GetObject())
	{}
	TScriptInterface<INotifyFieldValueChanged> GetSource() const
	{
		UObject* FoundObject = Source.Get();
		return FoundObject ? TScriptInterface<INotifyFieldValueChanged>(FoundObject) : TScriptInterface<INotifyFieldValueChanged>();
	}
};

/**
*
*/
struct FDependency
{
	UE::FieldNotification::FFieldId ParentFieldId;
	FSourceInstanceId ParentId;
	FSourceInstanceId ChildId;
	FViewModelBindings::FEvaluateSourceDelegate SourceDelegate;
};

/**
*
*/
struct FViewModelBindingsImpl
{
	~FViewModelBindingsImpl();

	FSource* FindSource(FSourceInstanceId ToEvaluate);
	FDependency* FindDependency(FSourceInstanceId ToEvaluate);
	void Test_ValidFieldId(TScriptInterface<INotifyFieldValueChanged> Source, UE::FieldNotification::FFieldId WhenField) const;

	FViewModelBindings::FBuilder AddSource(TScriptInterface<INotifyFieldValueChanged> Value);
	void RemoveSource(FSourceInstanceId Source);
	void SetSource(FSourceInstanceId Source, TScriptInterface<INotifyFieldValueChanged> Value);
	void AddBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate);
	void AddBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenFieldId, FSimpleDelegate Delegate);
	void RemoveAllBindings(FSourceInstanceId Source);
	void AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate);
	void AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, UE::FieldNotification::FFieldId WhenField, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate);
	void SortDependencies();
	void Execute();
	void Execute(FSourceInstanceId Source);
	void Evaluate(FSourceInstanceId Source);

private:
	void UnbindBindingsImpl(FSource* Source);
	void HandleFieldChanged(UObject* Source, UE::FieldNotification::FFieldId FieldId);

private:
	TMap<FSourceInstanceId, TUniquePtr<FSource>> Sources;
	TArray<TUniquePtr<FDependency>> Dependencies;
	bool bDirty_Dependency = true;
};

FViewModelBindingsImpl::~FViewModelBindingsImpl()
{
	// remove all bindings
	for (auto& SourcePair : Sources)
	{
		UnbindBindingsImpl(SourcePair.Value.Get());
	}
}

FSource* FViewModelBindingsImpl::FindSource(FSourceInstanceId SourceId)
{
	TUniquePtr<FSource>* Source = Sources.Find(SourceId);
	return Source ? Source->Get() : nullptr;
}

void FViewModelBindingsImpl::Test_ValidFieldId(TScriptInterface<INotifyFieldValueChanged> Interface, UE::FieldNotification::FFieldId WhenField) const
{
#if DO_CHECK
	if (ensure(Interface.GetInterface() && Interface.GetObject()))
	{
		UE::FieldNotification::FFieldId FoundFieldId = Interface->GetFieldNotificationDescriptor().GetField(Interface.GetObject()->GetClass(), WhenField.GetName());
		checkf(FoundFieldId.IsValid() && FoundFieldId == WhenField, TEXT("The FieldId is not part of that class."));
	}
#endif
}

FViewModelBindings::FBuilder FViewModelBindingsImpl::AddSource(TScriptInterface<INotifyFieldValueChanged> Value)
{
	if (Value.GetObject() && Value.GetInterface())
	{
		FSourceInstanceId SourceId = FSourceInstanceId::Create(Value.GetObject());
		if (FSource* FoundSource = FindSource(SourceId))
		{
			return FViewModelBindings::FBuilder(*this, SourceId);
		}
		else
		{
			bDirty_Dependency = true;
			Sources.Emplace(SourceId, MakeUnique<FSource>(Value));
			return FViewModelBindings::FBuilder(*this, SourceId);
		}
	}
	return FViewModelBindings::FBuilder(*this, FSourceInstanceId());
}

void FViewModelBindingsImpl::RemoveSource(FSourceInstanceId SourceId)
{
	TUniquePtr<FSource> Source;
	Sources.RemoveAndCopyValue(SourceId, Source);
	if (ensure(Source))
	{
		UnbindBindingsImpl(Source.Get());
	}

	for (int32 Index = Dependencies.Num() - 1; Index >= 0; --Index)
	{
		const FDependency* Dependency = Dependencies[Index].Get();
		if (Dependency->ParentId == SourceId)
		{
			Dependencies.RemoveAtSwap(Index);
			RemoveSource(Dependency->ChildId);
			Index = Dependencies.Num() - 1; // the recursive call can change the order. Restart the dependency algo.
		}
		else if (Dependency->ChildId == SourceId)
		{
			Dependencies.RemoveAtSwap(Index);
			bDirty_Dependency = true;
		}
	}
}

void FViewModelBindingsImpl::SetSource(FSourceInstanceId SourceId, TScriptInterface<INotifyFieldValueChanged> NewValue)
{
	if (FSource* FoundSource = FindSource(SourceId))
	{
		TScriptInterface<INotifyFieldValueChanged> PreviousValue = FoundSource->GetSource();
		if (PreviousValue == NewValue)
		{
			return;
		}

		// Remove the bindings
		UnbindBindingsImpl(FoundSource);

		// Set the value
		FoundSource->Source = NewValue.GetObject();

		// bind bindings
		if (NewValue.GetObject() && NewValue.GetInterface())
		{
			for (TUniquePtr<FBinding>& Binding : FoundSource->Bindings)
			{
				if (!FoundSource->Delegates.Contains(Binding->GetFieldId()))
				{
					FDelegateHandle Handle = NewValue->AddFieldValueChangedDelegate(Binding->GetFieldId(), INotifyFieldValueChanged::FFieldValueChangedDelegate::CreateRaw(this, &FViewModelBindingsImpl::HandleFieldChanged));
					FoundSource->Delegates.Emplace(Binding->GetFieldId(), Handle);
				}
			}
		}

		// Evaluate any dependencies
		SortDependencies();
		for (TUniquePtr<FDependency>& Dependency : Dependencies)
		{
			if (Dependency->ParentId == SourceId && !Dependency->ParentFieldId.IsValid())
			{
				UObject* NewDependencyValue = Dependency->SourceDelegate.IsBound() ? Dependency->SourceDelegate.Execute() : nullptr;
				SetSource(Dependency->ChildId, NewDependencyValue);
			}
		}

		// Execute all bindings from that source
		if (NewValue.GetObject() && NewValue.GetInterface())
		{
			Execute(SourceId);
		}
	}
}

void FViewModelBindingsImpl::UnbindBindingsImpl(FSource* Source)
{
	TScriptInterface<INotifyFieldValueChanged> Interface = Source->GetSource();
	if (Interface.GetObject() && Interface.GetInterface())
	{
		for (auto& DelegatePair : Source->Delegates)
		{
			Interface->RemoveFieldValueChangedDelegate(DelegatePair.Key, DelegatePair.Value);
		}
	}
	Source->Delegates.Reset();
}

void FViewModelBindingsImpl::AddBinding(FSourceInstanceId SourceId, UE::FieldNotification::FFieldId WhenFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate)
{
	FSource* Source = FindSource(SourceId);
	if (ensure(Source))
	{
		TScriptInterface<INotifyFieldValueChanged> Interface = Source->GetSource();
		Test_ValidFieldId(Interface, WhenFieldId);

		if (ensure(Interface.GetInterface() && Interface.GetObject()))
		{
			Source->Bindings.Add(MakeUnique<FBinding>(WhenFieldId, MoveTemp(Delegate)));
			if (!Source->Delegates.Contains(WhenFieldId))
			{
				FDelegateHandle Handle = Interface->AddFieldValueChangedDelegate(WhenFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate::CreateRaw(this, &FViewModelBindingsImpl::HandleFieldChanged));
				Source->Delegates.Emplace(WhenFieldId, Handle);
			}
		}
	}
}

void FViewModelBindingsImpl::AddBinding(FSourceInstanceId SourceId, UE::FieldNotification::FFieldId WhenFieldId, FSimpleDelegate Delegate)
{
	FSource* Source = FindSource(SourceId);
	if (ensure(Source))
	{
		TScriptInterface<INotifyFieldValueChanged> Interface = Source->GetSource();
		Test_ValidFieldId(Interface, WhenFieldId);

		if (ensure(Interface.GetInterface() && Interface.GetObject()))
		{
			Source->Bindings.Add(MakeUnique<FBinding>(WhenFieldId, MoveTemp(Delegate)));
			if (!Source->Delegates.Contains(WhenFieldId))
			{
				FDelegateHandle Handle = Interface->AddFieldValueChangedDelegate(WhenFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate::CreateRaw(this, &FViewModelBindingsImpl::HandleFieldChanged));
				Source->Delegates.Emplace(WhenFieldId, Handle);
			}
		}
	}
}

void FViewModelBindingsImpl::RemoveAllBindings(FSourceInstanceId SourceId)
{
	if (FSource* FoundSource = FindSource(SourceId))
	{
		UnbindBindingsImpl(FoundSource);
		FoundSource->Bindings.Empty();
		FoundSource->Delegates.Empty();
	}
}

void FViewModelBindingsImpl::AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate)
{
	bDirty_Dependency = true;
}

void FViewModelBindingsImpl::AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, UE::FieldNotification::FFieldId WhenField, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate)
{
	bDirty_Dependency = true;
}

void FViewModelBindingsImpl::SortDependencies()
{
	if (bDirty_Dependency)
	{
		// Sort dependency
		bDirty_Dependency = false;
	}
}

void FViewModelBindingsImpl::Execute()
{
	for (auto& SourcePair : Sources)
	{
		Execute(SourcePair.Key);
	}
}

void FViewModelBindingsImpl::Execute(FSourceInstanceId SourceId)
{
	if (ensure(SourceId.IsValid()))
	{
		SortDependencies();

		FSource* Source = FindSource(SourceId);
		if (ensure(Source))
		{
			// Run all bindings
			TScriptInterface<INotifyFieldValueChanged> Interface = Source->GetSource();
			if (ensure(Interface.GetInterface() && Interface.GetObject()))
			{
				for (TUniquePtr<FBinding>& Binding : Source->Bindings)
				{
					Binding->Execute(Interface.GetObject());
				}
			}
		}
	}
}

void FViewModelBindingsImpl::HandleFieldChanged(UObject* InSourceObject, UE::FieldNotification::FFieldId InFieldId)
{
	FSourceInstanceId SourceId = FSourceInstanceId::Create(InSourceObject);
	if (FSource* Source = FindSource(SourceId))
	{
		// Find FDependency
		SortDependencies();
		for (TUniquePtr<FDependency>& Dependency : Dependencies)
		{
			if (Dependency->ParentId == SourceId && InFieldId == Dependency->ParentFieldId)
			{
				UObject* NewValue = Dependency->SourceDelegate.IsBound() ? Dependency->SourceDelegate.Execute() : nullptr;
				SetSource(Dependency->ChildId, NewValue);
			}
		}

		// Find bindings
		check(Source->Delegates.Contains(InFieldId));
		for (TUniquePtr<FBinding>& Binding : Source->Bindings)
		{
			if (Binding->GetFieldId() == InFieldId)
			{
				Binding->Execute(InSourceObject);
			}
		}
	}
}

} //namespace

 
 /**
 * 
 */
FSourceInstanceId FSourceInstanceId::Create(const UObject* Object)
{
	FSourceInstanceId Result;
	Result.ObjectKey = FObjectKey(Object);
	return Result;
}

 /**
 * 
 */
FViewModelBindings::FBuilder::FBuilder(Private::FViewModelBindingsImpl& InInstance, FSourceInstanceId InId)
	: Instance(InInstance)
	, Id(InId)
{}

FViewModelBindings::FBuilder& FViewModelBindings::FBuilder::AddBinding(UE::FieldNotification::FFieldId WhenField, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate)
{
	Instance.AddBinding(GetId(), WhenField, MoveTemp(Delegate));
	return *this;
}

FViewModelBindings::FBuilder& FViewModelBindings::FBuilder::AddBinding(UE::FieldNotification::FFieldId WhenField, FSimpleDelegate Delegate)
{
	Instance.AddBinding(GetId(), WhenField, MoveTemp(Delegate));
	return *this;
}

FViewModelBindings::FBuilder& FViewModelBindings::FBuilder::AddDependency(FSourceInstanceId WhenChanged, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate)
{
	Instance.AddDependency(GetId(), WhenChanged, MoveTemp(EvaluateDelegate));
	return *this;
}

FViewModelBindings::FBuilder& FViewModelBindings::FBuilder::AddDependency(FSourceInstanceId WhenChanged, UE::FieldNotification::FFieldId WhenField, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate)
{
	Instance.AddDependency(GetId(), WhenChanged, WhenField, MoveTemp(EvaluateDelegate));
	return *this;
}

/**
 *
 */
FViewModelBindings::FViewModelBindings()
{
	Impl = MakePimpl<Private::FViewModelBindingsImpl>();
}

FViewModelBindings::FBuilder FViewModelBindings::AddSource(TScriptInterface<INotifyFieldValueChanged> Value)
{
	return Impl->AddSource(Value);
}

void FViewModelBindings::RemoveSource(FSourceInstanceId SourceId)
{}

void FViewModelBindings::SetSource(FSourceInstanceId, TScriptInterface<INotifyFieldValueChanged> Value)
{}

void FViewModelBindings::AddBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenField, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate)
{
	return Impl->AddBinding(Source, WhenField, MoveTemp(Delegate));
}

void FViewModelBindings::AddBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenField, FSimpleDelegate Delegate)
{
	return Impl->AddBinding(Source, WhenField, MoveTemp(Delegate));
}

void FViewModelBindings::RemoveAllBindings(FSourceInstanceId Source)
{
	Impl->RemoveAllBindings(Source);
}

void FViewModelBindings::AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate)
{
	Impl->AddDependency(ToEvaluate, WhenChanged, MoveTemp(EvaluateDelegate));
}

void FViewModelBindings::AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, UE::FieldNotification::FFieldId WhenField, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate)
{
	Impl->AddDependency(ToEvaluate, WhenChanged, WhenField, MoveTemp(EvaluateDelegate));
}

void FViewModelBindings::Execute()
{
	Impl->Execute();
}

void FViewModelBindings::Execute(FSourceInstanceId Source)
{
	Impl->Execute(Source);
}
} // namespace

 

//UCLASS()
//class UViewModelA: public UMVVMViewModelBase
//{
//	GENERATED_BODY()
//
//private:
//	UPROPERTY(BlueprintReadWrite, EditAnywhere, Setter, FieldNotify, Category = "ViewModel", meta = (AllowPrivateAccess))
//	int32 ValueA = 1;
//
//	UPROPERTY(BlueprintReadWrite, EditAnywhere, Setter, FieldNotify, Category = "ViewModel", meta = (AllowPrivateAccess))
//	TObjectPtr<UViewModelB> ViewModelB = nullptr;
//
//public:
//	void SetValueA(int32 InValue)
//	{
//		UE_MVVM_SET_PROPERTY_VALUE(ValueA, InValue);
//	}
//	int32 GetValueA() const
//	{
//		return ValueA;
//	}
//	void SetViewModelB(UViewModelB InValue)
//	{
//		UE_MVVM_SET_PROPERTY_VALUE(ViewModelB, InValue);
//	}
//	UViewModelB* GetViewModelB() const
//	{
//		return ViewModelB;
//	}
//};
//
//UCLASS()
//class UViewModelB : public UMVVMViewModelBase
//{
//	GENERATED_BODY()
//
//private:
//	UPROPERTY(BlueprintReadWrite, EditAnywhere, Setter, FieldNotify, Category = "ViewModel", meta = (AllowPrivateAccess))
//	int32 ValueB = 1;
//
//public:
//	void SetValueB(int32 InValue)
//	{
//		UE_MVVM_SET_PROPERTY_VALUE(ValueB, InValue);
//	}
//	int32 GetValueB() const
//	{
//		return Value;
//	}
//};
//
//
//class FMyEditor or SMyWidget
//{
//public:
//	void Init() or SMyWidget::Construct(const FArgument&)
//	{
//		Bindings = MakeUnique<UE::Slate::MVVM::FViewModelBindings>();
//
//		FSimpleDelegate ShareDelegate = FSimpleDelegate::CreateSP(this, &FMyEditor::HandleComplex);
//
//		A_Id = Bindings->AddSource(A.Get())
//			.AddBinding(UViewModelA::FFieldNotificationClassDescriptor::ValueA, ShareDelegate)
//			.AddBinding(UViewModelA::FFieldNotificationClassDescriptor::ValueA, FSimpleDelegate::CreateSP(this, &FMyEditor::HandleSimpleA))
//			.GetId();
//		Bindings->AddSource(A.GetViewModelB())
//			.AddBinding(UViewModelB::FFieldNotificationClassDescriptor::ValueB, ShareDelegate)
//			.AddBinding(UViewModelB::FFieldNotificationClassDescriptor::ValueB, FSimpleDelegate::CreateSP(this, &FMyEditor::HandleSimpleB));
//			.AddDependency(A_Id, UViewModelB::FFieldNotificationClassDescriptor::ViewModelB, FViewModelBindings::FEvaluateSourceDelegate::CreateUObject(A.Get(), &UViewModelA::GetViewModelB));
//
//		// or without the builder.
//		//Bindings->AddBinding(A_Id, UViewModelA::FFieldNotificationClassDescriptor::ValueA, FSimpleDelegate::CreateSP(this, &FMyEditor::HandleSimpleA));
//
//		//run all bindings once
//		Bindings->Evaluate();
//
//		// event like button event do not changes
//		Button.OnClicked.AddSP(this, &FMyEditor::HandleClicked);
//
//		@note the UMG version might decide to not execute this one because it knows that A or A->B can be nullptr.
//		void HandleSimpleA()
//		{
//			TextBlock.SetRenderOpacity(A->ValueA);
//		}
//
//		@note the UMG version might decide to not execute this one because it knows that A or A->B can be nullptr.
//		void HandleSimpleB()
//		{
//			TextBlock.SetRenderOpacity(A->B->ValueB);
//		}
//
//		@note the UMG version might decide to not execute this one because it knows that A or A->B can be nullptr.
//		void HandleComplex()
//		{
//			TextBlock.SetText(FText::Format("{0}.{1}"), A->ValueA, A->B->ValueB);
//		}
//	}
//
//	void Uninint()
//	{
//		Bindings.Reset(); // not needed, automatic because of TUniquePtr, but in case you want to...
//	}
//
//private:
//	TStrongObjectPtr<UMyViewmodelA> A;
//	TUniquePtr<UE::Slate::MVVM::FViewModelBindings> Bindings;
//};
