// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "FieldNotificationId.h"
#include "INotifyFieldValueChanged.h"
#include "UObject/ScriptInterface.h"
#include "UObject/ObjectKey.h"

namespace UE::Slate::MVVM
{

namespace Private { struct FViewModelBindingsImpl; }

struct FSourceInstanceId
{
	friend Private::FViewModelBindingsImpl;

public:
	FSourceInstanceId() = default;

	bool IsValid() const
	{
		return ObjectKey != FObjectKey();
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	bool operator ==(const FSourceInstanceId& Other) const
	{
		return ObjectKey == Other.ObjectKey;
	}

	friend uint32 GetTypeHash(const FSourceInstanceId& Key)
	{
		return GetTypeHash(Key.ObjectKey);
	}

private:
	static FSourceInstanceId Create(const UObject* Object);
	FObjectKey ObjectKey;
};

class FViewModelBindings
{
public:
	using FEvaluateSourceDelegate = TDelegate<UObject*()>;

public:
	struct FBuilder
	{
		FBuilder(Private::FViewModelBindingsImpl& Instance, FSourceInstanceId Id);
		FBuilder(const FBuilder&) = delete;
		FBuilder& operator=(const FBuilder&) = delete;

		SLATEMVVM_API FBuilder& AddBinding(UE::FieldNotification::FFieldId WhenFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate);
		SLATEMVVM_API FBuilder& AddBinding(UE::FieldNotification::FFieldId WhenFieldId, FSimpleDelegate Delegate);
		SLATEMVVM_API FBuilder& AddDependency(FSourceInstanceId WhenChanged, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate);
		SLATEMVVM_API FBuilder& AddDependency(FSourceInstanceId WhenChanged, UE::FieldNotification::FFieldId WhenFieldId, FViewModelBindings::FEvaluateSourceDelegate EvaluateDelegate);

		FSourceInstanceId GetId() const
		{
			return Id;
		}

	private:
		Private::FViewModelBindingsImpl& Instance;
		FSourceInstanceId Id;
	};

public:
	SLATEMVVM_API FViewModelBindings();
	FViewModelBindings(const FViewModelBindings&) = delete;
	FViewModelBindings& operator=(const FViewModelBindings&) = delete;

	/** Add a source that can provide notification when a value changes. */
	SLATEMVVM_API FBuilder AddSource(TScriptInterface<INotifyFieldValueChanged> Value);

	/**
	 * Add a source that can provide notification when a value changes.
	 * It will evaluate the dependencies.
	 */
	SLATEMVVM_API void RemoveSource(FSourceInstanceId Source);

	/**
	 * Set the new source value of a created source.
	 * It will evaluate the dependencies and execute the bindings of that source.
	 */
	SLATEMVVM_API void SetSource(FSourceInstanceId Source, TScriptInterface<INotifyFieldValueChanged> Value);

	/** Add a delegate to execute when a field changed. */
	SLATEMVVM_API void AddBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenFieldId, INotifyFieldValueChanged::FFieldValueChangedDelegate Delegate);

	/** Add a delegate to execute when a field changed. */
	SLATEMVVM_API void AddBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenFieldId, FSimpleDelegate Delegate);

	/** Remove a delegate. */
	//SLATEMVVM_API void RemoveBinding(FSourceInstanceId Source, UE::FieldNotification::FFieldId WhenFieldId);

	/** Remove all added delegates from that source. */
	SLATEMVVM_API void RemoveAllBindings(FSourceInstanceId Source);

	/**
	 * The source ToEvaluate depends on the source WhenChanged.
	 * When the WhenChanged value changes via SetSource, execute all ToEvaluate bindings.
	 * It may cascade to evaluate dependency of dependency.
	 */
	SLATEMVVM_API void AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, FEvaluateSourceDelegate EvaluateDelegate);

	/**
	 * The source ToEvaluate depends on the source WhenChanged.
	 * When the WhenChanged value changes via SetSource, execute all ToEvaluate bindings.
	 * Also, when the WhenField triggers, execute all ToEvalute bindings.
	 * It may cascade to evaluate dependency of dependency.
	 */
	SLATEMVVM_API void AddDependency(FSourceInstanceId ToEvaluate, FSourceInstanceId WhenChanged, UE::FieldNotification::FFieldId WhenFieldId, FEvaluateSourceDelegate EvaluateDelegate);

	/**
	 * Execute the bindings of all the sources.
	 * The Dependency will be executed only if the FieldId changes (executing a binding, trigger a modification that execute another binding).
	 */
	SLATEMVVM_API void Execute();

	/**
	 * Execute the bindings for that source.
	 * The Dependency will be executed only if the FieldId changes (executing a binding, trigger a modification that execute another binding).
	 */
	SLATEMVVM_API void Execute(FSourceInstanceId Source);

private:
	TPimplPtr<Private::FViewModelBindingsImpl> Impl;
};

} // namespace
